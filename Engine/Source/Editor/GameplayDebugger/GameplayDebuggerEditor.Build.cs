// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
	public class GameplayDebuggerEditor : ModuleRules
	{
		public GameplayDebuggerEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
				});

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"RenderCore",
					"InputCore",
					"SlateCore",
					"Slate",
					"DeveloperSettings",
				});
			
			if (Target.Configuration != UnrealTargetConfiguration.Shipping)
			{
				PrivateDependencyModuleNames.Add("DrawPrimitiveDebugger");
			}
/*
			PrivateIncludePaths.AddRange(
				new string[] {
					"Developer/Settings/Public",
				});
*/
			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[] {
						
						"EditorFramework",
						"UnrealEd",
						"LevelEditor",
						"PropertyEditor",
						"GameplayDebugger"
					});
			}

			if (Target.bUseGameplayDebuggerCore)
			{
				PrecompileForTargets = PrecompileTargetsType.Any;
			}
		}
	}
}
