// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerCategoryConfigCustomization.h"

#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Application/SlateWindowHelper.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "GameplayDebuggerConfig.h"

#define LOCTEXT_NAMESPACE "GameplayDebuggerConfig"

void FGameplayDebuggerCategoryConfigCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	CategoryNameProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerCategoryConfig, CategoryName));
	SlotIdxProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerCategoryConfig, SlotIdx));
	ActiveInGameProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerCategoryConfig, ActiveInGame));
	ActiveInSimulateProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerCategoryConfig, ActiveInSimulate));
	
	FSimpleDelegate RefreshHeaderDelegate = FSimpleDelegate::CreateSP(this, &FGameplayDebuggerCategoryConfigCustomization::OnChildValueChanged);
	StructPropertyHandle->SetOnChildPropertyValueChanged(RefreshHeaderDelegate);
	OnChildValueChanged();

	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent().VAlign(VAlign_Center).MinDesiredWidth(300.0f)
	[
		SNew(STextBlock)
		.Text(this, &FGameplayDebuggerCategoryConfigCustomization::GetHeaderDesc)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
};

void FGameplayDebuggerCategoryConfigCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildProps = 0;
	StructPropertyHandle->GetNumChildren(NumChildProps);

	for (uint32 Idx = 0; Idx < NumChildProps; Idx++)
	{
		TSharedPtr<IPropertyHandle> PropHandle = StructPropertyHandle->GetChildHandle(Idx);
		if (PropHandle->GetProperty() && PropHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FGameplayDebuggerCategoryConfig, CategoryName))
		{
			continue;
		}

		StructBuilder.AddProperty(PropHandle.ToSharedRef());
	}
};

void FGameplayDebuggerCategoryConfigCustomization::OnChildValueChanged()
{
	FString CategoryNameValue;
	if (CategoryNameProp.IsValid())
	{
		CategoryNameProp->GetValue(CategoryNameValue);
	}

	int32 SlotIdxValue = -1;
	if (SlotIdxProp.IsValid())
	{
		SlotIdxProp->GetValue(SlotIdxValue);
	}

	uint8 ActiveInGameValue = 0;
	if (ActiveInGameProp.IsValid())
	{
		ActiveInGameProp->GetValue(ActiveInGameValue);
	}

	uint8 ActiveInSimulateValue = 0;
	if (ActiveInSimulateProp.IsValid())
	{
		ActiveInSimulateProp->GetValue(ActiveInSimulateValue);
	}

	CachedHeader = FText::FromString(FString::Printf(TEXT("[%s]:%s%s%s"),
		SlotIdxValue < 0 ? TEXT("-") : *TTypeToString<int32>::ToString(SlotIdxValue),
		CategoryNameValue.Len() ? *CategoryNameValue : TEXT("??"),
		(ActiveInGameValue == (uint8)EGameplayDebuggerOverrideMode::UseDefault) ? TEXT("") :
			(ActiveInGameValue == (uint8)EGameplayDebuggerOverrideMode::Enable) ? TEXT(" game:ON") : TEXT(" game:OFF"),
		(ActiveInSimulateValue == (uint8)EGameplayDebuggerOverrideMode::UseDefault) ? TEXT("") :
			(ActiveInSimulateValue == (uint8)EGameplayDebuggerOverrideMode::Enable) ? TEXT(" simulate:ON") : TEXT(" simulate:OFF")
		));
}

#undef LOCTEXT_NAMESPACE
