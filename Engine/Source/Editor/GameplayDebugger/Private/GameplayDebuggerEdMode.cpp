// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerEdMode.h"

#include "CoreGlobals.h"
#include "GameplayDebuggerToolkit.h"
#include "Toolkits/ToolkitManager.h"
#include "GameplayDebuggerPlayerManager.h"
#include "GameplayDebuggerLocalController.h"

#include "EditorModeManager.h"
#include "Components/InputComponent.h"

const FName FGameplayDebuggerEdMode::EM_GameplayDebugger = TEXT("EM_GameplayDebugger");

void FGameplayDebuggerEdMode::Enter()
{
	FEdMode::Enter();
	
	if (!Toolkit.IsValid())
	{
		Toolkit = MakeShareable(new FGameplayDebuggerToolkit(this));
		Toolkit->Init(Owner->GetToolkitHost());
	}

	bPrevGAreScreenMessagesEnabled = GAreScreenMessagesEnabled;
	GAreScreenMessagesEnabled = false;
}

void FGameplayDebuggerEdMode::Exit()
{
	if (Toolkit.IsValid())
	{
		FToolkitManager::Get().CloseToolkit(Toolkit.ToSharedRef());
		Toolkit.Reset();
	}

	if (FocusedViewport)
	{
		EnableViewportClientFlags(FocusedViewport, false);
		FocusedViewport = nullptr;
	}

	FEdMode::Exit();
	GAreScreenMessagesEnabled = bPrevGAreScreenMessagesEnabled;
}

void FGameplayDebuggerEdMode::EnableViewportClientFlags(FEditorViewportClient* ViewportClient, bool bEnable)
{
	ViewportClient->bUseNumpadCameraControl = false;
}

bool FGameplayDebuggerEdMode::ReceivedFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	EnableViewportClientFlags(ViewportClient, true);
	FocusedViewport = ViewportClient;
	return false;
}

bool FGameplayDebuggerEdMode::LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport)
{
	EnableViewportClientFlags(ViewportClient, false);
	FocusedViewport = nullptr;
	return false;
}

bool FGameplayDebuggerEdMode::InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent)
{
#if WITH_GAMEPLAY_DEBUGGER
	UWorld* MyWorld = GetWorld();
	APlayerController* LocalPC = GEngine->GetFirstLocalPlayerController(MyWorld);

	if (LocalPC && InViewportClient)
	{
		// process raw input for debugger's input component manually
		// can't use LocalPC->InputKey here, since it will trigger for every bound chord, not only gameplay debugger ones
		// and will not work at all when simulation is paused

		AGameplayDebuggerPlayerManager& PlayerManager = AGameplayDebuggerPlayerManager::GetCurrent(MyWorld);
		const FGameplayDebuggerPlayerData* DataPtr = PlayerManager.GetPlayerData(*LocalPC);

		if (DataPtr && DataPtr->InputComponent && DataPtr->Controller && DataPtr->Controller->IsKeyBound(InKey.GetFName()))
		{
			const FInputChord ActiveChord(InKey, InViewportClient->IsShiftPressed(), InViewportClient->IsCtrlPressed(), InViewportClient->IsAltPressed(), InViewportClient->IsCmdPressed());

			// go over all bound actions
			const int32 NumBindings = DataPtr->InputComponent->KeyBindings.Num();
			for (int32 Idx = 0; Idx < NumBindings; Idx++)
			{
				const FInputKeyBinding& KeyBinding = DataPtr->InputComponent->KeyBindings[Idx];
				if (KeyBinding.KeyEvent == InEvent && KeyBinding.Chord == ActiveChord && KeyBinding.KeyDelegate.IsBound())
				{
					KeyBinding.KeyDelegate.Execute(InKey);
				}
			}

			return true;			
		}
	}
#endif // WITH_GAMEPLAY_DEBUGGER
	return false;
}

void FGameplayDebuggerEdMode::Tick(FEditorViewportClient* ViewportClient, float DeltaTime)
{
	if (ViewportClient == nullptr || !ViewportClient->EngineShowFlags.DebugAI)
	{
		Owner->DeactivateMode(FGameplayDebuggerEdMode::EM_GameplayDebugger);
	}
}

void FGameplayDebuggerEdMode::SafeOpenMode()
{
	// The global mode manager should not be created or accessed in a commandlet environment
	if (IsRunningCommandlet())
	{
		return;
	}

	// By calling this get, it make sure that the singleton is created and not during the SafeCloseMode whitch can be called during garbage collect.
	GLevelEditorModeTools();
}

void FGameplayDebuggerEdMode::SafeCloseMode()
{
	// The global mode manager should not be created or accessed in a commandlet environment
	if (IsRunningCommandlet())
	{
		return;
	}

	// this may be called on closing editor during PIE (~viewport -> teardown PIE -> debugger's cleanup on game end)
	//
	// DeactivateMode tries to bring up default mode, but toolkit is already destroyed by that time
	// and editor crashes on check in GLevelEditorModeTools().GetToolkitHost() inside default mode's code
	// note: no need to do it if IsEngineExitRequested since the GLevelEditorModeTools is no longer available 
	if (IsEngineExitRequested() == false && GLevelEditorModeTools().HasToolkitHost())
	{
		GLevelEditorModeTools().DeactivateMode(FGameplayDebuggerEdMode::EM_GameplayDebugger);
	}
}
