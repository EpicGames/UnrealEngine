// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerEditorModule.h"

#include "CoreMinimal.h"
#include "EditorModeManager.h"
#include "EditorModeRegistry.h"
#include "Engine/World.h"
#include "GameplayDebugger.h"
#include "GameplayDebuggerCategoryConfigCustomization.h"
#include "GameplayDebuggerEdMode.h"
#include "GameplayDebuggerExtensionConfigCustomization.h"
#include "GameplayDebuggerInputConfigCustomization.h"
#include "GameplayDebuggerModule.h"
#include "PropertyEditorModule.h"

IMPLEMENT_MODULE(FGameplayDebuggerEditorModule, GameplayDebuggerEditor)

void FGameplayDebuggerEditorModule::StartupModule()
{
	FGameplayDebuggerModule::OnLocalControllerInitialized.AddRaw(this, &FGameplayDebuggerEditorModule::OnLocalControllerInitialized);
	FGameplayDebuggerModule::OnLocalControllerUninitialized.AddRaw(this, &FGameplayDebuggerEditorModule::OnLocalControllerUninitialized);
	FGameplayDebuggerModule::OnDebuggerEdModeActivation.AddRaw(this, &FGameplayDebuggerEditorModule::OnDebuggerEdModeActivation);

	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("GameplayDebuggerCategoryConfig", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGameplayDebuggerCategoryConfigCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("GameplayDebuggerExtensionConfig", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGameplayDebuggerExtensionConfigCustomization::MakeInstance));
	PropertyEditorModule.RegisterCustomPropertyTypeLayout("GameplayDebuggerInputConfig", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FGameplayDebuggerInputConfigCustomization::MakeInstance));

	FEditorModeRegistry::Get().RegisterMode<FGameplayDebuggerEdMode>(FGameplayDebuggerEdMode::EM_GameplayDebugger);
}

void FGameplayDebuggerEditorModule::ShutdownModule()
{	
	FPropertyEditorModule* PropertyEditorModule = FModuleManager::GetModulePtr<FPropertyEditorModule>("PropertyEditor");
	if (PropertyEditorModule)
	{
		PropertyEditorModule->UnregisterCustomPropertyTypeLayout("GameplayDebuggerCategoryConfig");
		PropertyEditorModule->UnregisterCustomPropertyTypeLayout("GameplayDebuggerExtensionConfig");
		PropertyEditorModule->UnregisterCustomPropertyTypeLayout("GameplayDebuggerInputConfig");
	}

	FEditorModeRegistry::Get().UnregisterMode(FGameplayDebuggerEdMode::EM_GameplayDebugger);
}

void FGameplayDebuggerEditorModule::OnDebuggerEdModeActivation()
{
	GLevelEditorModeTools().ActivateMode(FGameplayDebuggerEdMode::EM_GameplayDebugger);
}

void FGameplayDebuggerEditorModule::OnLocalControllerInitialized()
{
	FGameplayDebuggerEdMode::SafeOpenMode();
}

void FGameplayDebuggerEditorModule::OnLocalControllerUninitialized()
{
	FGameplayDebuggerEdMode::SafeCloseMode();
}
