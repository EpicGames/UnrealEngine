// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerExtensionConfigCustomization.h"

#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Application/SlateWindowHelper.h"
#include "Widgets/Text/STextBlock.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "GameplayDebuggerConfig.h"

#define LOCTEXT_NAMESPACE "GameplayDebuggerConfig"

void FGameplayDebuggerExtensionConfigCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	ExtensionNameProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerExtensionConfig, ExtensionName));
	UseExtensionProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerExtensionConfig, UseExtension));
	
	FSimpleDelegate RefreshHeaderDelegate = FSimpleDelegate::CreateSP(this, &FGameplayDebuggerExtensionConfigCustomization::OnChildValueChanged);
	StructPropertyHandle->SetOnChildPropertyValueChanged(RefreshHeaderDelegate);
	OnChildValueChanged();

	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent().VAlign(VAlign_Center).MinDesiredWidth(300.0f)
	[
		SNew(STextBlock)
		.Text(this, &FGameplayDebuggerExtensionConfigCustomization::GetHeaderDesc)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
};

void FGameplayDebuggerExtensionConfigCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildProps = 0;
	StructPropertyHandle->GetNumChildren(NumChildProps);

	for (uint32 Idx = 0; Idx < NumChildProps; Idx++)
	{
		TSharedPtr<IPropertyHandle> PropHandle = StructPropertyHandle->GetChildHandle(Idx);
		if (PropHandle->GetProperty() && PropHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FGameplayDebuggerExtensionConfig, ExtensionName))
		{
			continue;
		}

		StructBuilder.AddProperty(PropHandle.ToSharedRef());
	}
};

void FGameplayDebuggerExtensionConfigCustomization::OnChildValueChanged()
{
	FString ExtensionNameValue;
	if (ExtensionNameProp.IsValid())
	{
		ExtensionNameProp->GetValue(ExtensionNameValue);
	}

	uint8 UseExtensionValue = 0;
	if (UseExtensionProp.IsValid())
	{
		UseExtensionProp->GetValue(UseExtensionValue);
	}

	CachedHeader = FText::FromString(FString::Printf(TEXT("%s %s"),
		ExtensionNameValue.Len() ? *ExtensionNameValue : TEXT("??"),
		(UseExtensionValue == (uint8)EGameplayDebuggerOverrideMode::UseDefault) ? TEXT("") :
			(UseExtensionValue == (uint8)EGameplayDebuggerOverrideMode::Enable) ? TEXT("is enabled") : TEXT("is disabled")
		));
}

#undef LOCTEXT_NAMESPACE
