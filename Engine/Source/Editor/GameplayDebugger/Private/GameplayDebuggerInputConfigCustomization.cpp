// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerInputConfigCustomization.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Engine/GameViewportClient.h"
#include "GameplayDebuggerTypes.h"

#include "Widgets/Text/STextBlock.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "DetailLayoutBuilder.h"
#include "GameplayDebuggerConfig.h"

#define LOCTEXT_NAMESPACE "GameplayDebuggerConfig"

void FGameplayDebuggerInputConfigCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	ConfigNameProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerInputConfig, ConfigName));
	KeyProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerInputConfig, Key));
	ModShiftProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerInputConfig, bModShift));
	ModCtrlProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerInputConfig, bModCtrl));
	ModAltProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerInputConfig, bModAlt));
	ModCmdProp = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FGameplayDebuggerInputConfig, bModCmd));

	FSimpleDelegate RefreshHeaderDelegate = FSimpleDelegate::CreateSP(this, &FGameplayDebuggerInputConfigCustomization::OnChildValueChanged);
	StructPropertyHandle->SetOnChildPropertyValueChanged(RefreshHeaderDelegate);
	OnChildValueChanged();

	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent().VAlign(VAlign_Center).MinDesiredWidth(300.0f)
	[
		SNew(STextBlock)
		.Text(this, &FGameplayDebuggerInputConfigCustomization::GetHeaderDesc)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
};

void FGameplayDebuggerInputConfigCustomization::CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildProps = 0;
	StructPropertyHandle->GetNumChildren(NumChildProps);

	for (uint32 Idx = 0; Idx < NumChildProps; Idx++)
	{
		TSharedPtr<IPropertyHandle> PropHandle = StructPropertyHandle->GetChildHandle(Idx);
		if (PropHandle->GetProperty() && PropHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FGameplayDebuggerInputConfig, ConfigName))
		{
			continue;
		}

		StructBuilder.AddProperty(PropHandle.ToSharedRef());
	}
};

void FGameplayDebuggerInputConfigCustomization::OnChildValueChanged()
{
	FString ConfigNameValue;
	if (ConfigNameProp.IsValid())
	{
		ConfigNameProp->GetValue(ConfigNameValue);
	}

	FString KeyValue;
	if (KeyProp.IsValid())
	{
		KeyProp->GetPerObjectValue(0, KeyValue);
	}

	bool bModShift = false;
	if (ModShiftProp.IsValid())
	{
		ModShiftProp->GetValue(bModShift);
	}

	bool bModCtrl = false;
	if (ModCtrlProp.IsValid())
	{
		ModCtrlProp->GetValue(bModCtrl);
	}
	
	bool bModAlt = false;
	if (ModAltProp.IsValid())
	{
		ModAltProp->GetValue(bModAlt);
	}
	
	bool bModCmd = false;
	if (ModCmdProp.IsValid())
	{
		ModCmdProp->GetValue(bModCmd);
	}

	FGameplayDebuggerInputHandler DummyHandler;
	DummyHandler.KeyName = *KeyValue;
	DummyHandler.Modifier.bShift = bModShift;
	DummyHandler.Modifier.bCtrl = bModCtrl;
	DummyHandler.Modifier.bAlt = bModAlt;
	DummyHandler.Modifier.bCmd = bModCmd;

	CachedHeader = FText::FromString(FString::Printf(TEXT("%s: %s"),
		ConfigNameValue.Len() ? *ConfigNameValue : TEXT("??"),
		*DummyHandler.ToString()
		));
}

#undef LOCTEXT_NAMESPACE
