// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerToolkit.h"
#include "Modules/ModuleManager.h"

#include "LevelEditor.h"
#include "IAssetViewport.h"
#include "SLevelViewport.h"
#include "LevelEditorViewport.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "GameplayDebuggerConfig"

FGameplayDebuggerToolkit::FGameplayDebuggerToolkit(class FEdMode* InOwningMode)
{
	DebuggerEdMode = InOwningMode;
}

FText FGameplayDebuggerToolkit::GetBaseToolkitName() const
{
	return LOCTEXT("EdToolTitle", "Gameplay Debugger");
}

FName FGameplayDebuggerToolkit::GetToolkitFName() const
{
	return FName("GameplayDebuggerToolkit");
}

void FGameplayDebuggerToolkit::Init(const TSharedPtr<class IToolkitHost>& InitToolkitHost)
{
	MyWidget =
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
		.Content()
		[
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EdToolStatus", "Gameplay debugger is active."))
				]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(0, 30.0f)
				[
					SNew(STextBlock)
					.Visibility(this, &FGameplayDebuggerToolkit::GetScreenMessageWarningVisibility)
					.Text(LOCTEXT("EdToolMessageWarning", "Warning! On screen messages are suppressed!\nUse EnableAllScreenMessages to restore them."))
				]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("EdToolDisableHint", "Clear DebugAI show flag to disable tool."))
				]
			+ SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.Padding(5.0f, 5.0f)
				[
					SNew(SButton)
					.Text(LOCTEXT("EdToolDisableButton", "Disable tool"))
					.OnClicked(this, &FGameplayDebuggerToolkit::OnClickedDisableTool)
				]
		];

	FModeToolkit::Init(InitToolkitHost);
}

EVisibility FGameplayDebuggerToolkit::GetScreenMessageWarningVisibility() const
{
	return GAreScreenMessagesEnabled ? EVisibility::Collapsed : EVisibility::Visible;
}

FReply FGameplayDebuggerToolkit::OnClickedDisableTool()
{
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	TSharedPtr<ILevelEditor> LevelEditor = LevelEditorModule.GetFirstLevelEditor();
	if (LevelEditor.IsValid())
	{
		TArray<TSharedPtr<SLevelViewport>> Viewports = LevelEditor->GetViewports();
		for (const TSharedPtr<SLevelViewport>& ViewportWindow : Viewports)
		{
			if (ViewportWindow.IsValid())
			{
				FEditorViewportClient& Viewport = ViewportWindow->GetAssetViewportClient();
				Viewport.EngineShowFlags.SetDebugAI(false);
			}
		}
	}
	
	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE
