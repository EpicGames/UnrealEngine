// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "InputCoreTypes.h"
#include "EditorModes.h"
#include "EdMode.h"
#include "EditorViewportClient.h"

class FGameplayDebuggerEdMode : public FEdMode
{
public:
	FGameplayDebuggerEdMode() : FocusedViewport(nullptr) {}

	// FEdMode interface
	virtual bool UsesToolkits() const override { return true; }
	virtual void Enter() override;
	virtual void Exit() override;
	virtual bool ReceivedFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	virtual bool LostFocus(FEditorViewportClient* ViewportClient, FViewport* Viewport) override;
	virtual bool InputKey(FEditorViewportClient* InViewportClient, FViewport* InViewport, FKey InKey, EInputEvent InEvent) override;
	virtual void Tick(FEditorViewportClient* ViewportClient, float DeltaTime);
	virtual bool RequiresLegacyViewportInteractions() const override { return false; }
	// End of FEdMode interface

	static void SafeOpenMode();
	static void SafeCloseMode();

	static const FName EM_GameplayDebugger;

private:
	FEditorViewportClient* FocusedViewport;
	bool bPrevGAreScreenMessagesEnabled;

	void EnableViewportClientFlags(FEditorViewportClient* ViewportClient, bool bEnable);
};
