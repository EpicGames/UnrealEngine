// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/World.h"
#include "Modules/ModuleManager.h"
#include "Modules/ModuleInterface.h"

class FGameplayDebuggerEditorModule : public IModuleInterface
{
public:
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	void OnWorldInitialized(UWorld* World, const UWorld::InitializationValues IVS);

	void OnLocalControllerInitialized();
	void OnLocalControllerUninitialized();
	void OnDebuggerEdModeActivation();
};
