// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;

class FGameplayDebuggerExtensionConfigCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FGameplayDebuggerExtensionConfigCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	TSharedPtr<IPropertyHandle> ExtensionNameProp;
	TSharedPtr<IPropertyHandle> UseExtensionProp;

	FText CachedHeader;
	FText GetHeaderDesc() const { return CachedHeader; }

	void OnChildValueChanged();
};
