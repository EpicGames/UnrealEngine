// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "IPropertyTypeCustomization.h"

class IPropertyHandle;

class FGameplayDebuggerInputConfigCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FGameplayDebuggerInputConfigCustomization);
	}

	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<class IPropertyHandle> StructPropertyHandle, class IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	TSharedPtr<IPropertyHandle> ConfigNameProp;
	TSharedPtr<IPropertyHandle> KeyProp;
	TSharedPtr<IPropertyHandle> ModShiftProp;
	TSharedPtr<IPropertyHandle> ModCtrlProp;
	TSharedPtr<IPropertyHandle> ModAltProp;
	TSharedPtr<IPropertyHandle> ModCmdProp;

	FText CachedHeader;
	FText GetHeaderDesc() const { return CachedHeader; }

	void OnChildValueChanged();
};
