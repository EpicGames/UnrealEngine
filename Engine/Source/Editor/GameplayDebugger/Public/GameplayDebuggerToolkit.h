// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/SWidget.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/BaseToolkit.h"

class FGameplayDebuggerToolkit : public FModeToolkit
{
public:
	FGameplayDebuggerToolkit(class FEdMode* InOwningMode);

	// IToolkit interface
	virtual FText GetBaseToolkitName() const override;
	virtual FName GetToolkitFName() const override;
	virtual class FEdMode* GetEditorMode() const override { return DebuggerEdMode; }
	virtual TSharedPtr<class SWidget> GetInlineContent() const override { return MyWidget; }
	// End of IToolkit interface

	// FModeToolkit interface
	virtual void Init(const TSharedPtr<class IToolkitHost>& InitToolkitHost) override;
	// End of FModeToolkit interface

private:
	class FEdMode* DebuggerEdMode;
	TSharedPtr<class SWidget> MyWidget;

	EVisibility GetScreenMessageWarningVisibility() const;
	FReply OnClickedDisableTool();
};
